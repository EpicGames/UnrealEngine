// Copyright Epic Games, Inc. All Rights Reserved.

#include "Containers/SharedString.h"
#include "GenericPlatform/GenericPlatformStackWalk.h"
#include "Misc/DefinePrivateMemberPtr.h"
#include "Misc/Paths.h"
#include "ImageCore.h"
#include "Modules/ModuleManager.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "TextureCompressorModule.h"
#include "PixelFormat.h"
#include "HAL/PlatformProcess.h"
#include "TextureBuildFunction.h"
#include "DerivedDataBuildFunctionFactory.h"

#ifndef __APPLE__
#define __APPLE__ 0
#endif
#ifndef __unix__
#define __unix__ 0
#endif
#include "Etc.h"
#include "EtcErrorMetric.h"
#include "EtcBlock4x4.h"
#include "EtcImage.h"

// Workaround for: error LNK2019: unresolved external symbol __imp___std_init_once_begin_initialize referenced in function "void __cdecl std::call_once
// https://developercommunity.visualstudio.com/t/-imp-std-init-once-complete-unresolved-external-sy/1684365
#if defined(_MSC_VER) && (_MSC_VER >= 1932)  // Visual Studio 2022 version 17.2+
#    pragma comment(linker, "/alternatename:__imp___std_init_once_complete=__imp_InitOnceComplete")
#    pragma comment(linker, "/alternatename:__imp___std_init_once_begin_initialize=__imp_InitOnceBeginInitialize")
#endif

DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatETC2, Log, All);

class FETC2TextureBuildFunction final : public FTextureBuildFunction
{
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(UTF8TEXTVIEW("ETC2Texture"));
		return Name;
	}

	void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const final
	{
		static FGuid Version(TEXT("af5192f4-351f-422f-b539-f6bd4abadfae"));
		Builder << Version;
		OutTextureFormatVersioning = FModuleManager::GetModuleChecked<ITextureFormatModule>(TEXT("TextureFormatETC2")).GetTextureFormat();
	}
};

/**
 * Macro trickery for supported format names.
 */
#define ENUM_SUPPORTED_FORMATS(op) \
	op(ETC2_RGB) \
	op(ETC2_RGBA) \
	op(ETC2_R11) \
	op(ETC2_RG11) \
	op(AutoETC2)

#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
#undef DECL_FORMAT_NAME

#define DECL_FORMAT_NAME_ENTRY(FormatName) GTextureFormatName##FormatName ,
static FName GSupportedTextureFormatNames[] =
{
	ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME_ENTRY)
};
#undef DECL_FORMAT_NAME_ENTRY

#undef ENUM_SUPPORTED_FORMATS

// note InSourceData is not const, can be mutated by sanitize
static bool CompressImageUsingEtc2comp(
	FLinearColor * InSourceColors,
	EPixelFormat PixelFormat,
	int32 SizeX,
	int32 SizeY,
	int64 NumPixels,
	EGammaSpace TargetGammaSpace,
	TArray64<uint8>& OutCompressedData)
{
	using namespace Etc;
		
	Image::Format EtcFormat = Image::Format::UNKNOWN;
	switch (PixelFormat)
	{
	case PF_ETC2_RGB:
		EtcFormat = Image::Format::RGB8;
		break;
	case PF_ETC2_RGBA:
		EtcFormat = Image::Format::RGBA8;
		break;
	case PF_ETC2_R11_EAC:
		EtcFormat = Image::Format::R11;
		break;
	case PF_ETC2_RG11_EAC:
		EtcFormat = Image::Format::RG11;
		break;
	default:
		UE_LOGF(LogTextureFormatETC2, Fatal, "Unsupported EPixelFormat for compression: %u", (uint32)PixelFormat);
		return false;
	}

	// RGBA, REC709, NUMERIC will set RGB to 0 if all pixels in the block are transparent (A=0)
	const Etc::ErrorMetric EtcErrorMetric = Etc::RGBX;
	const float EtcEffort = ETCCOMP_DEFAULT_EFFORT_LEVEL;
	// threads used by etc2comp :
	const unsigned int MAX_JOBS = 8;
	const unsigned int NUM_JOBS = 8;
	// to run etc2comp synchronously :
	//const unsigned int MAX_JOBS = 0;
	//const unsigned int NUM_JOBS = 0;

	unsigned char* paucEncodingBits = nullptr;
	unsigned int uiEncodingBitsBytes = 0;
	unsigned int uiExtendedWidth = 0;
	unsigned int uiExtendedHeight = 0;
	int iEncodingTime_ms = 0;
	float* SourceData = &InSourceColors[0].Component(0);
	
	// InSourceData is a linear color, we need to feed float* data to the codec in a target color space
	TArray64<float> IntermediateData;
	if (TargetGammaSpace == EGammaSpace::sRGB)
	{
		IntermediateData.Reserve(NumPixels * 4);
		IntermediateData.AddUninitialized(NumPixels * 4);

		for (int64 Idx = 0; Idx < IntermediateData.Num(); Idx += 4)
		{
			const FLinearColor& LinColor = *(FLinearColor*)(SourceData + Idx);
			FColor Color = LinColor.ToFColorSRGB();
			IntermediateData[Idx + 0] = Color.R / 255.f;
			IntermediateData[Idx + 1] = Color.G / 255.f;
			IntermediateData[Idx + 2] = Color.B / 255.f;
			IntermediateData[Idx + 3] = Color.A / 255.f;
		}
		
		SourceData = IntermediateData.GetData();
	}
	else
	{
		int64 NumFloats = NumPixels * 4;

		for(int64 Idx =0 ;Idx < NumFloats;Idx++)
		{
			// sanitize inf and nan :
			float f = SourceData[Idx];
			if ( f >= -FLT_MAX && f <= FLT_MAX )
			{
				// finite, leave it
				// nans will fail all compares so not go in here
			}
			else if ( f > FLT_MAX )
			{
				// +inf
				SourceData[Idx] = FLT_MAX;
			}
			else if ( f < -FLT_MAX )
			{
				// -inf
				SourceData[Idx] = -FLT_MAX;
			}
			else
			{
				// nan
				SourceData[Idx] = 0.f;
			}

			//check( ! FMath::IsNaN( SourceData[Idx] ) );
		}
	}

	Encode(
		SourceData,
		SizeX, SizeY,
		EtcFormat,
		EtcErrorMetric,
		EtcEffort,
		NUM_JOBS,
		MAX_JOBS,
		&paucEncodingBits, &uiEncodingBitsBytes,
		&uiExtendedWidth, &uiExtendedHeight,
		&iEncodingTime_ms
	);

	OutCompressedData.SetNumUninitialized(uiEncodingBitsBytes);
	FMemory::Memcpy(OutCompressedData.GetData(), paucEncodingBits, uiEncodingBitsBytes);
	delete[] paucEncodingBits;
	return true;
}

UE_DEFINE_PRIVATE_MEMBER_PTR(Etc::Image::Format, GPrivateFormatPtr, Etc::Image, m_format);

/**
 * ETC2 texture format handler.
 */
class FTextureFormatETC2 : public ITextureFormat
{
public:
	static FGuid GetDecodeBuildFunctionVersionGuid()
	{
		static FGuid Version(TEXT("B1C15A49-199A-4CD0-8F03-E19FB13292C2"));
		return Version;
	}
	static FUtf8StringView GetDecodeBuildFunctionNameStatic()
	{
		return UTF8TEXTVIEW("FDecodeTextureFormatETC2");
	}
	virtual const FUtf8StringView GetDecodeBuildFunctionName() const override final
	{
		return GetDecodeBuildFunctionNameStatic();
	}


	virtual bool AllowParallelBuild() const override
	{
		return true;
	}

	virtual uint16 GetVersion(
		FName Format,
		const struct FTextureBuildSettings* BuildSettings = nullptr
	) const override
	{
		return 3;
	}

	virtual FName GetEncoderName(FName Format) const override
	{
		static const FName ETC2Name("ETC2");
		return ETC2Name;
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Append(GSupportedTextureFormatNames, UE_ARRAY_COUNT(GSupportedTextureFormatNames));
	}

	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& BuildSettings, bool bImageHasAlphaChannel) const override
	{
		if (BuildSettings.TextureFormatName == GTextureFormatNameETC2_RGB ||
			BuildSettings.TextureFormatName == GTextureFormatNameETC2_RGBA ||
			BuildSettings.TextureFormatName == GTextureFormatNameAutoETC2 )
		{
			if ( BuildSettings.TextureFormatName == GTextureFormatNameETC2_RGB || !bImageHasAlphaChannel )
			{
				// even if Name was RGBA we still use the RGB profile if !bImageHasAlphaChannel
				//	so that "Compress Without Alpha" can force us to opaque

				return PF_ETC2_RGB;
			}
			else
			{
				return PF_ETC2_RGBA;
			}
		}

		if (BuildSettings.TextureFormatName == GTextureFormatNameETC2_R11)
		{
			return PF_ETC2_R11_EAC;
		}
		else if (BuildSettings.TextureFormatName == GTextureFormatNameETC2_RG11)
		{
			return PF_ETC2_RG11_EAC;
		}

		UE_LOGF(LogTextureFormatETC2, Fatal, "Unhandled texture format '%ls' given to FTextureFormatAndroid::GetEncodedPixelFormat()", *BuildSettings.TextureFormatName.ToString());
		return PF_Unknown;
	}

	virtual bool CanDecodeFormat(EPixelFormat InPixelFormat) const
	{
		return IsETCBlockCompressedPixelFormat(InPixelFormat);
	}

	virtual bool DecodeImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, EPixelFormat InPixelFormat, bool bInSRGB, const FName& InTextureFormatName, FSharedBuffer InEncodedData, FImage& OutImage, FStringView InTextureName) const
	{
		Etc::Image::Format EtcFormat;
		switch (InPixelFormat)
		{
		case PF_ETC2_RGBA: EtcFormat = Etc::Image::Format::RGBA8; break;
		case PF_ETC2_RGB:  EtcFormat = Etc::Image::Format::RGB8; break;
		case PF_ETC2_R11_EAC: EtcFormat = Etc::Image::Format::R11; break;
		case PF_ETC2_RG11_EAC: EtcFormat = Etc::Image::Format::RG11; break;
		default: check(0); return false; // should never get here because of CanDecodeFormat()
		}

		uint32 BytesPerBlock = GPixelFormats[InPixelFormat].BlockBytes;
		check(BytesPerBlock == 16 || BytesPerBlock == 8);

		uint64 PitchInPixels = InSizeX;
		uint64 NumBlocksX = (InSizeX + 3) >> 2;
		uint64 NumBlocksY = (InSizeY + 3) >> 2;
		uint64 NumSlices = InNumSlices;
		uint64 BytesPerSlice = BytesPerBlock * NumBlocksX * NumBlocksY;

		if (NumSlices * NumBlocksX * NumBlocksY * BytesPerBlock != InEncodedData.GetSize())
		{
			UE_LOG(LogTextureFormatETC2, Error, TEXT("Can't decode ETC2 image: incorrect amount of encoded data for image size: %d x %d x %" UINT64_FMT ", expected %" UINT64_FMT " got %" UINT64_FMT),
				InSizeX, InSizeY, NumSlices, NumSlices * NumBlocksX * NumBlocksY * BytesPerBlock, InEncodedData.GetSize());
			return false;
		}

		// Etc actually alters the source image based on format and actually looks at the bits so they have to be valid even if they aren't
		// representative.
		TArray64<uint8> GarbageSourceBits;
		GarbageSourceBits.AddUninitialized(InSizeX * InSizeY * sizeof(FLinearColor));
		Etc::Image SourceImage((float*)GarbageSourceBits.GetData(), InSizeX, InSizeY, Etc::ErrorMetric::RGBA);

		// Annoyingly, there doesn't appear to be a way to set the image format during decoding - even using the encoded bits constructor with 
		// the actual format parameter doesn't matter because the relevant assert is looking at the source image which has format unknown. So
		// we edit the header to make this member public:
		SourceImage.*GPrivateFormatPtr = EtcFormat; // this is the same as SourceImage.m_format = EtcFormat;

		// This is so we don't have to allocate a full sized full linear color image - we copy into a 4x4 image and then blit the bits
		// back out.
		FImage LinearImage(4, 4, 1, ERawImageFormat::RGBA32F, EGammaSpace::Linear);
		FImage FColorImage(4, 4, 1, ERawImageFormat::BGRA8, EGammaSpace::Linear);

		FLinearColor* LinearBlock = (FLinearColor*)LinearImage.RawData.GetData();
		FColor* FColorBlock = (FColor*)FColorImage.RawData.GetData();

		OutImage.Init(InSizeX, InSizeY, InNumSlices, ERawImageFormat::BGRA8, EGammaSpace::Linear);

		// \todo profile this and see if we can do anything or if we're just boned by the interface.
		for (uint64 Slice = 0; Slice < NumSlices; Slice++)
		{
			for (uint64 BlockY = 0; BlockY < NumBlocksY; BlockY++)
			{
				for (uint64 BlockX = 0; BlockX < NumBlocksX; BlockX++)
				{
					uint64 BlockOffset = BytesPerBlock * (BlockY * NumBlocksX + BlockX) + BytesPerSlice * Slice;
					if (BlockOffset + BytesPerBlock > InEncodedData.GetSize())
					{
						UE_LOGF(LogTextureFormatETC2, Error, "Invalid block offset calculated during DecodeImage: %llu + %d, have %llu bytes available. Texture %.*ls", BlockOffset, BytesPerBlock, InEncodedData.GetSize(), InTextureName.Len(), InTextureName.GetData());
						UE_LOGF(LogTextureFormatETC2, Error, "....Slice %llu BlockX %llu BlockY %llu NumBlocksX %llu NumBlocksY %llu BytesPerSlice %llu", Slice, BlockX, BlockY, NumBlocksX, NumBlocksY, BytesPerSlice);
						return false;
					}
					const uint8* BlockBits = (uint8*)InEncodedData.GetData() + BlockOffset;
				
					Etc::Block4x4 Block;
					Block.InitFromEtcEncodingBits(EtcFormat, BlockX * 4, BlockY * 4, (uint8*)BlockBits, &SourceImage, Etc::ErrorMetric::RGBA);

					// Decode the color into a small 4x4 linear block
					Etc::ColorFloatRGBA* DecodedColors = Block.GetDecodedColors();
					for (uint64 PixelX = 0; PixelX < 4; PixelX++)
					{
						for (uint64 PixelY = 0; PixelY < 4; PixelY++)
						{
							LinearBlock[PixelY * 4 + PixelX].R = DecodedColors[PixelX * 4 + PixelY].fR;
							LinearBlock[PixelY * 4 + PixelX].G = DecodedColors[PixelX * 4 + PixelY].fG;
							LinearBlock[PixelY * 4 + PixelX].B = DecodedColors[PixelX * 4 + PixelY].fB;
							LinearBlock[PixelY * 4 + PixelX].A = 1.0f;
						}
					}

					if (InPixelFormat == PF_ETC2_RGBA ||
						InPixelFormat == PF_ETC2_RG11_EAC) // could have punchthrough alpha
					{
						float* DecodedAlphas = Block.GetDecodedAlphas();
						for (uint64 PixelX = 0; PixelX < 4; PixelX++)
						{
							for (uint64 PixelY = 0; PixelY < 4; PixelY++)
							{
								LinearBlock[PixelY * 4 + PixelX].A = DecodedAlphas[PixelX * 4 + PixelY];
							}
						}
					}

					// Convert to our output format
					LinearImage.CopyTo(FColorImage, ERawImageFormat::BGRA8, EGammaSpace::Linear);

					// Now copy these bits in to the actual output image
					{
						FColor* OutputBlock = (FColor*)OutImage.GetPixelPointer(BlockX * 4, BlockY * 4, Slice);

						uint64 BlockPixelsW = 4;
						uint64 BlockPixelsH = 4;
						if (BlockX == NumBlocksX - 1)
						{
							BlockPixelsW = InSizeX - BlockX * 4;
						}
						if (BlockY == NumBlocksY - 1)
						{
							BlockPixelsH = InSizeY - BlockY * 4;
						}

						for (uint64 PixelY = 0; PixelY < BlockPixelsH; PixelY++)
						{
							FMemory::Memcpy(OutputBlock + PixelY * PitchInPixels, FColorBlock + PixelY * 4, sizeof(FColor)*BlockPixelsW);
						}
					} // end copy to output
				} // end each horiz block
			} // end each vert block
		} // end each slice

		return true;
	}

	virtual bool CompressImage(
		const FImage& InImage,
		const struct FTextureBuildSettings& BuildSettings,
		const FIntVector3& InMip0Dimensions,
		int32 InMip0NumSlicesNoDepth,
		int32 InMipIndex,
		int32 InMipCount,
		FStringView DebugTexturePathName,
		bool bImageHasAlphaChannel,
		FCompressedImage2D& OutCompressedImage
		) const override
	{
		const FImage& Image = InImage;
		// Source is expected to be F32 linear color
		check(Image.Format == ERawImageFormat::RGBA32F);

		EPixelFormat CompressedPixelFormat = GetEncodedPixelFormat(BuildSettings, bImageHasAlphaChannel);

		bool bCompressionSucceeded = true;
		int64 SliceSize = Image.GetSliceNumPixels();
		for (int32 SliceIndex = 0; SliceIndex < Image.NumSlices && bCompressionSucceeded; ++SliceIndex)
		{
			TArray64<uint8> CompressedSliceData;
			
			const FLinearColor * SlicePixels = Image.AsRGBA32F().GetData() + SliceIndex * SliceSize;
			bCompressionSucceeded = CompressImageUsingEtc2comp(
				const_cast<FLinearColor *>(SlicePixels),
				CompressedPixelFormat,
				Image.SizeX,
				Image.SizeY,
				SliceSize,
				BuildSettings.GetDestGammaSpace(),
				CompressedSliceData
			);
			OutCompressedImage.RawData.Append(CompressedSliceData);
		}

		if (bCompressionSucceeded)
		{
			OutCompressedImage.SizeX = Image.SizeX;
			OutCompressedImage.SizeY = Image.SizeY;
			OutCompressedImage.NumSlicesWithDepth = Image.NumSlices;
			OutCompressedImage.PixelFormat = CompressedPixelFormat;
		}

		return bCompressionSucceeded;
	}
};

class FTextureFormatETC2Module : public ITextureFormatModule
{
public:
	ITextureFormat* Singleton = NULL;

	FTextureFormatETC2Module() { }
	virtual ~FTextureFormatETC2Module()
	{
		if ( Singleton )
		{
			delete Singleton;
			Singleton = nullptr;
		}
	}

	virtual void StartupModule() override
	{
	}
	
	virtual bool CanCallGetTextureFormats() override { return false; }

	virtual ITextureFormat* GetTextureFormat()
	{
		if ( Singleton == nullptr )  // not thread safe
		{
			FTextureFormatETC2* ptr = new FTextureFormatETC2();
			Singleton = ptr;
		}
		return Singleton;
	}

	static inline UE::DerivedData::TBuildFunctionFactory<FETC2TextureBuildFunction> BuildFunctionFactory;
	static inline UE::DerivedData::TBuildFunctionFactory<FGenericTextureDecodeBuildFunction<FTextureFormatETC2>> DecodeBuildFunctionFactory;
};

IMPLEMENT_MODULE(FTextureFormatETC2Module, TextureFormatETC2);
