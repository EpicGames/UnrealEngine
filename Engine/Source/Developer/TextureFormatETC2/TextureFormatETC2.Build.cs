// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormatETC2 : ModuleRules
{
	public TextureFormatETC2(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"TextureCompressor",
			"TextureFormat",
		});

		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"DerivedDataCache",
			"ImageCore",
			"TextureBuild",
		});

		AddEngineThirdPartyPrivateStaticDependencies(Target, "etc2comp");
	}
}
